// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetworkFileSystem : ModuleRules
	{
		public NetworkFileSystem(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"Projects",
					"SandboxFile",
					"TargetPlatform",
					"DesktopPlatform",
					"CookOnTheFly",
					"CookOnTheFlyNetServer"
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"Sockets",
				});

			PrecompileForTargets = PrecompileTargetsType.Editor;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
