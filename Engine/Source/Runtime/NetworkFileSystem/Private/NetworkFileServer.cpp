// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkFileServer.h"
#include "Misc/OutputDeviceRedirector.h"
#include "IPAddress.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "NetworkMessage.h"
#include "NetworkFileSystemLog.h"
#include "NetworkFileServerConnection.h"
#include "CookOnTheFlyNetServer.h"
#include "CookOnTheFly.h"
#include "CookOnTheFlyMessages.h"

class FCookOnTheFlyNetworkFileServerConnection
	: public FNetworkFileServerClientConnection
{
public:
	FCookOnTheFlyNetworkFileServerConnection(UE::Cook::ICookOnTheFlyClientConnection& InConnection, const FNetworkFileServerOptions& InFileServerOptions)
		: FNetworkFileServerClientConnection(InFileServerOptions)
		, Connection(InConnection)
	{
	}

	bool ProcessRequest(const UE::Cook::FCookOnTheFlyRequest& Request)
	{
		using namespace UE::Cook;
		using namespace UE::ZenCookOnTheFly::Messaging;

		RequestToRespondTo = &Request;
		TUniquePtr<FArchive> RequestBody = Request.ReadBody();
		return ProcessPayload(*RequestBody);
	}

	virtual bool SendPayload(TArray<uint8>& Out) override
	{
		using namespace UE::Cook;
		using namespace UE::ZenCookOnTheFly::Messaging;

		if (RequestToRespondTo)
		{
			FCookOnTheFlyResponse Response(*RequestToRespondTo);
			RequestToRespondTo = nullptr;
			TUniquePtr<FArchive> PayloadWriter = Response.WriteBody();
			PayloadWriter->Serialize(Out.GetData(), Out.Num());
			return Connection.SendMessage(Response);
		}
		else
		{
			FCookOnTheFlyMessage Message(ECookOnTheFlyMessage::NetworkPlatformFile);
			TUniquePtr<FArchive> PayloadWriter = Message.WriteBody();
			PayloadWriter->Serialize(Out.GetData(), Out.Num());
			return Connection.SendMessage(Message);
		}
	}

private:
	UE::Cook::ICookOnTheFlyClientConnection& Connection;
	const UE::Cook::FCookOnTheFlyRequest* RequestToRespondTo = nullptr;
};

/* FNetworkFileServer constructors
 *****************************************************************************/

FNetworkFileServer::FNetworkFileServer(FNetworkFileServerOptions InFileServerOptions, TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> InCookOnTheFlyNetworkServer)
	: FileServerOptions(MoveTemp(InFileServerOptions))
	, CookOnTheFlyServer(InCookOnTheFlyNetworkServer)
{
	using namespace UE::Cook;

	CookOnTheFlyServer->OnClientConnected().AddRaw(this, &FNetworkFileServer::OnClientConnected);
	CookOnTheFlyServer->OnClientDisconnected().AddRaw(this, &FNetworkFileServer::OnClientDisconnected);
	CookOnTheFlyServer->OnRequest(ECookOnTheFlyMessage::NetworkPlatformFile).BindRaw(this, &FNetworkFileServer::HandleRequest);
}

FNetworkFileServer::~FNetworkFileServer()
{
	Shutdown();
}

void FNetworkFileServer::OnClientConnected(UE::Cook::ICookOnTheFlyClientConnection& Connection)
{
	FCookOnTheFlyNetworkFileServerConnection* InternalConnection = new FCookOnTheFlyNetworkFileServerConnection(Connection, FileServerOptions);
	FScopeLock _(&ConnectionsCritical);
	Connections.Add(&Connection, InternalConnection);
}

void FNetworkFileServer::OnClientDisconnected(UE::Cook::ICookOnTheFlyClientConnection& Connection)
{
	FNetworkFileServerClientConnection* InternalConnection;
	{
		FScopeLock _(&ConnectionsCritical);
		InternalConnection = Connections.FindRef(&Connection);
		Connections.Remove(&Connection);
	}
	delete InternalConnection;
}

bool FNetworkFileServer::HandleRequest(UE::Cook::ICookOnTheFlyClientConnection& Connection, const UE::Cook::FCookOnTheFlyRequest& Request)
{
	FCookOnTheFlyNetworkFileServerConnection* InternalConnection;
	{
		FScopeLock _(&ConnectionsCritical);
		InternalConnection = Connections.FindRef(&Connection);
	}
	if (!InternalConnection)
	{
		return false;
	}
	return InternalConnection->ProcessRequest(Request);
}


/* INetworkFileServer overrides
 *****************************************************************************/

FString FNetworkFileServer::GetSupportedProtocol() const
{
	return CookOnTheFlyServer->GetSupportedProtocol();
}


bool FNetworkFileServer::GetAddressList( TArray<TSharedPtr<FInternetAddr> >& OutAddresses ) const
{
	return CookOnTheFlyServer->GetAddressList(OutAddresses);
}


bool FNetworkFileServer::IsItReadyToAcceptConnections(void) const
{
	return CookOnTheFlyServer->IsReadyToAcceptConnections();
}

int32 FNetworkFileServer::NumConnections(void) const
{
	return CookOnTheFlyServer->NumConnections();
}

void FNetworkFileServer::Shutdown(void)
{
	if (!CookOnTheFlyServer)
	{
		return;
	}
	CookOnTheFlyServer->OnClientConnected().RemoveAll(this);
	CookOnTheFlyServer->OnClientDisconnected().RemoveAll(this);
	CookOnTheFlyServer->OnRequest(UE::Cook::ECookOnTheFlyMessage::NetworkPlatformFile).Unbind();
	CookOnTheFlyServer.Reset();
}
