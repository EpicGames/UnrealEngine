// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/Runnable.h"
#include "INetworkFileServer.h"
#include "INetworkFileSystemModule.h"

class FInternetAddr;
class FSocket;
class ITargetPlatform;
namespace UE::Cook
{
	class ICookOnTheFlyNetworkServer;
	class ICookOnTheFlyClientConnection;
	class FCookOnTheFlyRequest;
}

/**
 * This class wraps the server thread and network connection
 */
class FNetworkFileServer
	: public INetworkFileServer
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFileServerOptions Network file server options
	 */
	FNetworkFileServer(FNetworkFileServerOptions InFileServerOptions, TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> InCookOnTheFlyNetworkServer);

	/**
	 * Destructor.
	 */
	~FNetworkFileServer( );

	// INetworkFileServer interface

	virtual bool IsItReadyToAcceptConnections(void) const override;
	virtual bool GetAddressList(TArray<TSharedPtr<FInternetAddr> >& OutAddresses) const override;
	virtual FString GetSupportedProtocol() const override;
	virtual int32 NumConnections() const override;
	virtual void Shutdown() override;
private:
	void OnClientConnected(UE::Cook::ICookOnTheFlyClientConnection& Connection);
	void OnClientDisconnected(UE::Cook::ICookOnTheFlyClientConnection& Connection);
	bool HandleRequest(UE::Cook::ICookOnTheFlyClientConnection& Connection, const UE::Cook::FCookOnTheFlyRequest& Request);

	// File server options
	FNetworkFileServerOptions FileServerOptions;

	TSharedPtr<UE::Cook::ICookOnTheFlyNetworkServer> CookOnTheFlyServer;
	
	// Holds all the client connections.
	FCriticalSection ConnectionsCritical;
	TMap<UE::Cook::ICookOnTheFlyClientConnection*, class FCookOnTheFlyNetworkFileServerConnection*> Connections;
};
