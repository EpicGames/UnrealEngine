// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkFileServerConnection.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Serialization/BufferArchive.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "IPlatformFileSandboxWrapper.h"
#include "NetworkMessage.h"
#include "ProjectDescriptor.h"
#include "NetworkFileSystemLog.h"
#include "Misc/PackageName.h"
#include "Interfaces/ITargetPlatform.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/IPluginManager.h"
#include "PlatformInfo.h"


/**
 * Helper function for resolving engine and game sandbox paths
 */
void GetSandboxRootDirectories(FSandboxPlatformFile* Sandbox, FString& SandboxEngine, FString& SandboxProject, FString& SandboxEnginePlatformExtensions, FString& SandboxProjectPlatformExtensions, const FString& LocalEngineDir, const FString& LocalProjectDir, const FString& LocalEnginePlatformExtensionsDir, const FString& LocalProjectPlatformExtensionsDir)
{
	SandboxEngine = Sandbox->ConvertToSandboxPath(*LocalEngineDir);
	if (SandboxEngine.EndsWith(TEXT("/"), ESearchCase::CaseSensitive) == false)
	{
		SandboxEngine += TEXT("/");
	}

	// we need to add an extra bit to the game path to make the sandbox convert it correctly (investigate?)
	// @todo: double check this
	SandboxProject = Sandbox->ConvertToSandboxPath(*(LocalProjectDir + TEXT("a.txt"))).Replace(TEXT("a.txt"), TEXT(""));
	SandboxEnginePlatformExtensions = Sandbox->ConvertToSandboxPath(*(LocalEnginePlatformExtensionsDir + TEXT("a.txt"))).Replace(TEXT("a.txt"), TEXT(""));
	SandboxProjectPlatformExtensions = Sandbox->ConvertToSandboxPath(*(LocalProjectPlatformExtensionsDir + TEXT("a.txt"))).Replace(TEXT("a.txt"), TEXT(""));
}

static FString MakeAbsoluteNormalizedDir(const FString& InPath)
{
	FString Out = FPaths::ConvertRelativePathToFull(InPath);
	if (Out.EndsWith(TEXT("/")))
	{
		Out.RemoveAt(Out.Len() - 1, EAllowShrinking::No);
	}
	return Out;
}

struct FSandboxOnlyScope
{
	FSandboxOnlyScope(FSandboxPlatformFile& InSandbox, bool bInSandboxOnly)
		: Sandbox(InSandbox)
	{
		Sandbox.SetSandboxOnly(bInSandboxOnly);
	}

	~FSandboxOnlyScope()
	{
		Sandbox.SetSandboxOnly(false);
	}

	FSandboxPlatformFile& Sandbox;
};

// These are marked unsafe because they do not work with Programs. However, COTF is unlikely to be used with Programs
// These are also temporary until some issues can be debugged
static FString UnsafeEnginePlatformExtensionDir()
{
	return FPaths::EnginePlatformExtensionDir(TEXT("")).TrimChar('/');
}

static FString UnsafeProjectPlatformExtensionDir()
{
	return FPaths::ProjectPlatformExtensionDir(TEXT("")).TrimChar('/');
}

/* FNetworkFileServerClientConnection structors
 *****************************************************************************/

FNetworkFileServerClientConnection::FNetworkFileServerClientConnection(const FNetworkFileServerOptions& Options)
	: LastHandleId(0)
	, Sandbox(NULL)
	, NetworkFileDelegates(&Options.Delegates)
	, ActiveTargetPlatforms(Options.TargetPlatforms)
	, bRestrictPackageAssetsToSandbox(Options.bRestrictPackageAssetsToSandbox)
{	
	//stats
	FileRequestDelegateTime = 0.0;
	PackageFileTime = 0.0;
	UnsolicitedFilesTime = 0.0;

	FileRequestCount = 0;
	UnsolicitedFilesCount = 0;
	PackageRequestsSucceeded = 0;
	PackageRequestsFailed = 0;
	FileBytesSent = 0;

	if ( NetworkFileDelegates && NetworkFileDelegates->OnFileModifiedCallback )
	{
		NetworkFileDelegates->OnFileModifiedCallback->AddRaw(this, &FNetworkFileServerClientConnection::FileModifiedCallback);
	}

	LocalEngineDir = FPaths::EngineDir();
	LocalProjectDir = FPaths::ProjectDir();
	LocalEnginePlatformExtensionsDir = UnsafeEnginePlatformExtensionDir();
	LocalProjectPlatformExtensionsDir = UnsafeProjectPlatformExtensionDir();

	if (FPaths::IsProjectFilePathSet())
	{
		LocalProjectDir = FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/");
		FPaths::MakeStandardFilename(LocalProjectDir);
	}

	LocalEngineDirAbs = MakeAbsoluteNormalizedDir(LocalEngineDir);
	LocalProjectDirAbs = MakeAbsoluteNormalizedDir(LocalProjectDir);
	LocalEnginePlatformExtensionsDirAbs = MakeAbsoluteNormalizedDir(LocalEnginePlatformExtensionsDir);
	LocalProjectPlatformExtensionsDirAbs = MakeAbsoluteNormalizedDir(LocalEnginePlatformExtensionsDir);
}


FNetworkFileServerClientConnection::~FNetworkFileServerClientConnection( )
{
	if (NetworkFileDelegates && NetworkFileDelegates->OnFileModifiedCallback)
	{
		NetworkFileDelegates->OnFileModifiedCallback->RemoveAll(this);
	}

	// close all the files the client had opened through us when the client disconnects
	for (TMap<uint64, IFileHandle*>::TIterator It(OpenFiles); It; ++It)
	{
		delete It.Value();
	}
}

static bool TrySubstituteDirectory(FString& FilenameToConvert, const FString& Directory, const FString& DirectoryToReplace)
{
	FString NormalizedFilenameToConvert = FilenameToConvert;
	FPaths::NormalizeFilename(NormalizedFilenameToConvert);
	FString NormalizedDirectoryToReplace = DirectoryToReplace;
	FPaths::NormalizeDirectoryName(NormalizedDirectoryToReplace);
	if (NormalizedFilenameToConvert.StartsWith(NormalizedDirectoryToReplace) && (NormalizedFilenameToConvert.Len() == NormalizedDirectoryToReplace.Len() || NormalizedFilenameToConvert[NormalizedDirectoryToReplace.Len()] == '/'))
	{
		if (NormalizedFilenameToConvert.Len() > NormalizedDirectoryToReplace.Len())
		{
			FilenameToConvert = Directory / NormalizedFilenameToConvert.RightChop(NormalizedDirectoryToReplace.Len() + 1);
		}
		else
		{
			FilenameToConvert = Directory;
		}
		return true;
	}
	return false;
}

/* FStreamingNetworkFileServerConnection implementation
 *****************************************************************************/
void FNetworkFileServerClientConnection::ConvertClientFilenameToServerFilename(FString& FilenameToConvert)
{
	if (TrySubstituteDirectory(FilenameToConvert, FPaths::EngineDir(), ConnectedEngineDir))
	{
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvert, FPaths::IsProjectFilePathSet() ? LocalProjectDir : (IS_PROGRAM ? ConnectedProjectDir : FPaths::ProjectDir()), ConnectedProjectDir))
	{
		// We have set the replacement value argument of TrySubstituteDirectory to be the same as the search value in the IS_PROGRAM case. We do this because:
		// UnrealFileServer has a ProjectDir of ../../../Engine/Programs/UnrealFileServer.
		// We do *not* want to replace the directory in that case.
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvert, UnsafeEnginePlatformExtensionDir(), ConnectedEnginePlatformExtensionsDir))
	{
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvert, UnsafeProjectPlatformExtensionDir(), ConnectedProjectPlatformExtensionsDir))
	{
		return;
	}
}

void FNetworkFileServerClientConnection::ConvertLocalFilenameToServerFilename(FString& FilenameToConvert)
{
	FString FilenameToConvertAbs = FPaths::ConvertRelativePathToFull(FilenameToConvert);
	if (TrySubstituteDirectory(FilenameToConvertAbs, LocalEngineDir, LocalEngineDirAbs))
	{
		FilenameToConvert = FilenameToConvertAbs;
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvertAbs, LocalProjectDir, LocalProjectDirAbs))
	{
		FilenameToConvert = FilenameToConvertAbs;
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvertAbs, LocalEnginePlatformExtensionsDir, LocalEnginePlatformExtensionsDirAbs))
	{
		FilenameToConvert = FilenameToConvertAbs;
		return;
	}
	if (TrySubstituteDirectory(FilenameToConvertAbs, LocalProjectPlatformExtensionsDir, LocalProjectPlatformExtensionsDirAbs))
	{
		FilenameToConvert = FilenameToConvertAbs;
		return;
	}
}


/**
 * Fixup sandbox paths to match what package loading will request on the client side.  e.g.
 * Sandbox path: "../../../Elemental/Content/Elemental/Effects/FX_Snow_Cracks/Crack_02/Materials/M_SnowBlast.uasset ->
 * client path: "../../../Samples/Showcases/Elemental/Content/Elemental/Effects/FX_Snow_Cracks/Crack_02/Materials/M_SnowBlast.uasset"
 * This ensures that devicelocal-cached files will be properly timestamp checked before deletion.
 */
TMap<FString, FDateTime> FNetworkFileServerClientConnection::FixupSandboxPathsForClient(const TMap<FString, FDateTime>& SandboxPaths)
{
	TMap<FString,FDateTime> FixedFiletimes;

	// since the sandbox remaps from A/B/C to C, and the client has no idea of this, we need to put the files
	// into terms of the actual LocalProjectDir, which is all that the client knows about
	for (TMap<FString, FDateTime>::TConstIterator It(SandboxPaths); It; ++It)
	{
		FixedFiletimes.Add(FixupSandboxPathForClient(It.Key()), It.Value());
	}
	return FixedFiletimes;
}

/**
 * Fixup sandbox paths to match what package loading will request on the client side.  e.g.
 * Sandbox path: "../../../Elemental/Content/Elemental/Effects/FX_Snow_Cracks/Crack_02/Materials/M_SnowBlast.uasset ->
 * client path: "../../../Samples/Showcases/Elemental/Content/Elemental/Effects/FX_Snow_Cracks/Crack_02/Materials/M_SnowBlast.uasset"
 * This ensures that devicelocal-cached files will be properly timestamp checked before deletion.
 */
FString FNetworkFileServerClientConnection::FixupSandboxPathForClient(const FString& Filename)
{
	FString Fixed = Sandbox->ConvertToSandboxPath(*Filename);
	Fixed = Fixed.Replace(*SandboxEngine, *LocalEngineDir);
	Fixed = Fixed.Replace(*SandboxProject, *LocalProjectDir);
	Fixed = Fixed.Replace(*SandboxEnginePlatformExtensions, *LocalEnginePlatformExtensionsDir);
	Fixed = Fixed.Replace(*SandboxProjectPlatformExtensions, *LocalProjectPlatformExtensionsDir);

	if (bSendLowerCase)
	{
		Fixed = Fixed.ToLower();
	}
	return Fixed;
}

static void ConvertServerFilenameToClientFilename(FString& FilenameToConvert, const FString& ConnectedEngineDir, const FString& ConnectedProjectDir)
{

	if (FilenameToConvert.StartsWith(FPaths::EngineDir()))
	{
		FilenameToConvert = FilenameToConvert.Replace(*(FPaths::EngineDir()), *ConnectedEngineDir);
	}
	else if (FPaths::IsProjectFilePathSet())
	{
		if (FilenameToConvert.StartsWith(FPaths::GetPath(FPaths::GetProjectFilePath())))
		{
			FilenameToConvert = FilenameToConvert.Replace(*(FPaths::GetPath(FPaths::GetProjectFilePath()) + TEXT("/")), *ConnectedProjectDir);
		}
	}
#if !IS_PROGRAM
	else if (FilenameToConvert.StartsWith(FPaths::ProjectDir()))
	{
			// UnrealFileServer has a ProjectDir of ../../../Engine/Programs/UnrealFileServer.
			// We do *not* want to replace the directory in that case.
			FilenameToConvert = FilenameToConvert.Replace(*(FPaths::ProjectDir()), *ConnectedProjectDir);
	}
#endif
}

static FCriticalSection SocketCriticalSection;

bool FNetworkFileServerClientConnection::ProcessPayload(FArchive& Ar)
{
	FBufferArchive Out;
	bool Result = true;

	// first part of the payload is always the command
	uint32 Cmd;
	Ar << Cmd;

	UE_LOGF(LogFileServer, Verbose, "Processing payload with Cmd %d", Cmd);

	// what type of message is this?
	NFS_Messages::Type Msg = NFS_Messages::Type(Cmd);

	// make sure the first thing is GetFileList which initializes the game/platform
	checkf(Msg == NFS_Messages::GetFileList || Msg == NFS_Messages::Heartbeat || Sandbox != NULL, TEXT("The first client message MUST be GetFileList, not %d"), (int32)Msg);

	// process the message!
	bool bSendUnsolicitedFiles = false;

	{
		FScopeLock SocketLock(&SocketCriticalSection);

		switch (Msg)
		{
		case NFS_Messages::OpenRead:
			ProcessOpenFile(Ar, Out, false);
			break;

		case NFS_Messages::OpenWrite:
			ProcessOpenFile(Ar, Out, true);
			break;

		case NFS_Messages::Read:
			ProcessReadFile(Ar, Out);
			break;

		case NFS_Messages::Write:
			ProcessWriteFile(Ar, Out);
			break;

		case NFS_Messages::Seek:
			ProcessSeekFile(Ar, Out);
			break;

		case NFS_Messages::Close:
			ProcessCloseFile(Ar, Out);
			break;

		case NFS_Messages::MoveFile:
			ProcessMoveFile(Ar, Out);
			break;

		case NFS_Messages::DeleteFile:
			ProcessDeleteFile(Ar, Out);
			break;

		case NFS_Messages::GetFileInfo:
			ProcessGetFileInfo(Ar, Out);
			break;

		case NFS_Messages::CopyFile:
			ProcessCopyFile(Ar, Out);
			break;

		case NFS_Messages::SetTimeStamp:
			ProcessSetTimeStamp(Ar, Out);
			break;

		case NFS_Messages::SetReadOnly:
			ProcessSetReadOnly(Ar, Out);
			break;

		case NFS_Messages::CreateDirectory:
			ProcessCreateDirectory(Ar, Out);
			break;

		case NFS_Messages::DeleteDirectory:
			ProcessDeleteDirectory(Ar, Out);
			break;

		case NFS_Messages::DeleteDirectoryRecursively:
			ProcessDeleteDirectoryRecursively(Ar, Out);
			break;

		case NFS_Messages::ToAbsolutePathForRead:
			ProcessToAbsolutePathForRead(Ar, Out);
			break;

		case NFS_Messages::ToAbsolutePathForWrite:
			ProcessToAbsolutePathForWrite(Ar, Out);
			break;

		case NFS_Messages::ReportLocalFiles:
			ProcessReportLocalFiles(Ar, Out);
			break;

		case NFS_Messages::GetFileList:
			Result = ProcessGetFileList(Ar, Out);
			break;

		case NFS_Messages::Heartbeat:
			ProcessHeartbeat(Ar, Out);
			break;

		case NFS_Messages::SyncFile:
			ProcessSyncFile(Ar, Out);
			bSendUnsolicitedFiles = true;
			break;

		default:

			UE_LOGF(LogFileServer, Error, "Bad incomming message tag (%d).", (int32)Msg);
		}
	}


	// send back a reply if the command wrote anything back out
	if (Out.Num() && Result )
	{
		int32 NumUnsolictedFiles = 0;


		if (bSendUnsolicitedFiles)
		{
			int64 MaxMemoryAllowed = 50 * 1024 * 1024;
			for (const auto& Filename : UnsolictedFiles)
			{
				// get file timestamp and send it to client
				FDateTime ServerTimeStamp = Sandbox->GetTimeStamp(*Filename);

				TArray<uint8> Contents;
				// open file
				int64 FileSize = Sandbox->FileSize(*Filename);

				if (MaxMemoryAllowed > FileSize)
				{
					MaxMemoryAllowed -= FileSize;
					++NumUnsolictedFiles;
				}
			}
			Out << NumUnsolictedFiles;
		}
		
		UE_LOGF(LogFileServer, Verbose, "Returning payload with %d bytes", Out.Num());

		// send back a reply
		Result &= SendPayload( Out );

		TArray<FString> UnprocessedUnsolictedFiles;
		UnprocessedUnsolictedFiles.Empty(NumUnsolictedFiles);

		if (bSendUnsolicitedFiles && Result )
		{
			double StartTime;
			StartTime = FPlatformTime::Seconds();
			for (int32 Index = 0; Index < NumUnsolictedFiles; Index++)
			{
				FBufferArchive OutUnsolicitedFile;
				ConvertLocalFilenameToServerFilename(UnsolictedFiles[Index]);
				FString TargetFilename = UnsolictedFiles[Index];
				ConvertServerFilenameToClientFilename(TargetFilename, ConnectedEngineDir, ConnectedProjectDir);
				PackageFile(UnsolictedFiles[Index], TargetFilename, OutUnsolicitedFile);

				UE_LOGF(LogFileServer, Display, "Returning unsolicited file %ls with %d bytes", *UnsolictedFiles[Index], OutUnsolicitedFile.Num());

				Result &= SendPayload(OutUnsolicitedFile);
				++UnsolicitedFilesCount;
			}
			UnsolictedFiles.RemoveAt(0, NumUnsolictedFiles);


			UnsolicitedFilesTime += 1000.0f * float(FPlatformTime::Seconds() - StartTime);
		}
	}

	UE_LOG(LogFileServer, Verbose, TEXT("Done Processing payload with Cmd %d Total Size sending %" INT64_FMT " "), Cmd,Out.TotalSize());

	return Result;
}


void FNetworkFileServerClientConnection::ProcessOpenFile( FArchive& In, FArchive& Out, bool bIsWriting )
{
	// Get filename
	FString Filename;
	In << Filename;

	bool bAppend = false;
	bool bAllowRead = false;

	if (bIsWriting)
	{
		In << bAppend;
		In << bAllowRead;
	}

	// todo: clients from the same ip address "could" be trying to write to the same file in the same sandbox (for example multiple windows clients)
	//			should probably have the sandbox write to separate files for each client
	//			not important for now

	ConvertClientFilenameToServerFilename(Filename);

	if (bIsWriting)
	{
		// Make sure the directory exists...
		Sandbox->CreateDirectoryTree(*(FPaths::GetPath(Filename)));
	}

	TArray<FString> NewUnsolictedFiles;
	NetworkFileDelegates->FileRequestDelegate.ExecuteIfBound(Filename, ConnectedPlatformName, NewUnsolictedFiles);

	// Disable access to outside the sandbox to prevent sending uncooked packages to the client
	const bool bSandboxOnly = bRestrictPackageAssetsToSandbox && !bIsWriting && FPackageName::IsPackageExtension(*FPaths::GetExtension(Filename, true));
	FSandboxOnlyScope _(*Sandbox, bSandboxOnly);

	FDateTime ServerTimeStamp = Sandbox->GetTimeStamp(*Filename);
	int64 ServerFileSize = 0;
	IFileHandle* File = bIsWriting ? Sandbox->OpenWrite(*Filename, bAppend, bAllowRead) : Sandbox->OpenRead(*Filename);
	if (!File)
	{
		UE_LOGF(LogFileServer, Display, "Open request for %ls failed for file %ls.", bIsWriting ? TEXT("Writing") : TEXT("Reading"), *Filename);
		ServerTimeStamp = FDateTime::MinValue(); // if this was a directory, this will make sure it is not confused with a zero byte file
	}
	else
	{
		ServerFileSize = File->Size();
	}

	uint64 HandleId = ++LastHandleId;
	OpenFiles.Add( HandleId, File );
	
	Out << HandleId;
	Out << ServerTimeStamp;
	Out << ServerFileSize;
}


void FNetworkFileServerClientConnection::ProcessReadFile( FArchive& In, FArchive& Out )
{
	// Get Handle ID
	uint64 HandleId = 0;
	In << HandleId;

	int64 BytesToRead = 0;
	In << BytesToRead;

	IFileHandle* File = FindOpenFile(HandleId);

	if (File)
	{
		constexpr int64 BufferSize = 4 << 20;
		uint8* Buffer = (uint8*)FMemory::Malloc(BufferSize);
		bool bIsFirstRead = true;
		while (BytesToRead > 0)
		{
			int64 CappedBytesToRead = FMath::Min(BufferSize, BytesToRead);
			if (!File->Read(Buffer, CappedBytesToRead))
			{
				if (bIsFirstRead)
				{
					int64 BytesRead = 0;
					Out << BytesRead;
					break;
				}
				// If this is not the first read we've already written the expected number of bytes to the stream so we have to deliver on that
				FMemory::Memset(Buffer, 0, CappedBytesToRead);
			}
			else if (bIsFirstRead)
			{
				Out << BytesToRead;
			}
			Out.Serialize(Buffer, CappedBytesToRead);
			BytesToRead -= CappedBytesToRead;
			bIsFirstRead = false;
			
		}
		FMemory::Free(Buffer);
	}
	else
	{
		int64 BytesRead = 0;
		Out << BytesRead;
	}
}


void FNetworkFileServerClientConnection::ProcessWriteFile( FArchive& In, FArchive& Out )
{
	// Get Handle ID
	uint64 HandleId = 0;
	In << HandleId;

	int64 BytesWritten = 0;
	IFileHandle* File = FindOpenFile(HandleId);

	if (File)
	{
		int64 BytesToWrite = 0;
		In << BytesToWrite;

		constexpr int64 BufferSize = 4 << 20;
		uint8* Buffer = (uint8*)FMemory::Malloc(BufferSize);
		while (BytesToWrite > 0)
		{
			int64 CappedBytesToWrite = FMath::Min(BufferSize, BytesToWrite);
			In.Serialize(Buffer, CappedBytesToWrite);
			if (!File->Write(Buffer, CappedBytesToWrite))
			{
				break;
			}
			BytesWritten += CappedBytesToWrite;
			BytesToWrite -= CappedBytesToWrite;
		}
		FMemory::Free(Buffer);
	}
		
	Out << BytesWritten;
}


void FNetworkFileServerClientConnection::ProcessSeekFile( FArchive& In, FArchive& Out )
{
	// Get Handle ID
	uint64 HandleId = 0;
	In << HandleId;

	int64 NewPosition;
	In << NewPosition;

	int64 SetPosition = -1;
	IFileHandle* File = FindOpenFile(HandleId);

	if (File && File->Seek(NewPosition))
	{
		SetPosition = File->Tell();
	}

	Out << SetPosition;
}


void FNetworkFileServerClientConnection::ProcessCloseFile( FArchive& In, FArchive& Out )
{
	// Get Handle ID
	uint64 HandleId = 0;
	In << HandleId;

	uint32 Closed = 0;
	IFileHandle* File = FindOpenFile(HandleId);

	if (File)
	{
		Closed = 1;
		OpenFiles.Remove(HandleId);

		delete File;
	}
		
	Out << Closed;
}


void FNetworkFileServerClientConnection::ProcessGetFileInfo( FArchive& In, FArchive& Out )
{
	// Get filename
	FString Filename;
	In << Filename;

	ConvertClientFilenameToServerFilename(Filename);

	FFileInfo Info;
	Info.FileExists = Sandbox->FileExists(*Filename);

	// if the file exists, cook it if necessary (the FileExists flag won't change value based on this callback)
	// without this, the server can return the uncooked file size, which can cause reads off the end
	if (Info.FileExists)
	{
		TArray<FString> NewUnsolictedFiles;
		NetworkFileDelegates->FileRequestDelegate.ExecuteIfBound(Filename, ConnectedPlatformName, NewUnsolictedFiles);
	}

	// get the rest of the info
	Info.ReadOnly = Sandbox->IsReadOnly(*Filename);
	Info.Size = Sandbox->FileSize(*Filename);
	Info.TimeStamp = Sandbox->GetTimeStamp(*Filename);
	Info.AccessTimeStamp = Sandbox->GetAccessTimeStamp(*Filename);

	Out << Info.FileExists;
	Out << Info.ReadOnly;
	Out << Info.Size;
	Out << Info.TimeStamp;
	Out << Info.AccessTimeStamp;
}


void FNetworkFileServerClientConnection::ProcessMoveFile( FArchive& In, FArchive& Out )
{
	FString From;
	In << From;
	FString To;
	In << To;

	ConvertClientFilenameToServerFilename(From);
	ConvertClientFilenameToServerFilename(To);

	uint32 Success = Sandbox->MoveFile(*To, *From);
	Out << Success;
}


void FNetworkFileServerClientConnection::ProcessDeleteFile( FArchive& In, FArchive& Out )
{
	FString Filename;
	In << Filename;

	ConvertClientFilenameToServerFilename(Filename);

	uint32 Success = Sandbox->DeleteFile(*Filename);
	Out << Success;
}


void FNetworkFileServerClientConnection::ProcessReportLocalFiles( FArchive& In, FArchive& Out )
{
	// get the list of files on the other end
	TMap<FString, FDateTime> ClientFileTimes;
	In << ClientFileTimes;

	// go over them and compare times to this side
	TArray<FString> OutOfDateFiles;

	for (TMap<FString, FDateTime>::TIterator It(ClientFileTimes); It; ++It)
	{
		FString ClientFile = It.Key();
		ConvertClientFilenameToServerFilename(ClientFile);
		// get the local timestamp
		FDateTime Timestamp = Sandbox->GetTimeStamp(*ClientFile);

		// if it's newer than the client/remote timestamp, it's newer here, so tell the other side it's out of date
		if (Timestamp > It.Value())
		{
			OutOfDateFiles.Add(ClientFile);
		}
	}

	UE_LOGF(LogFileServer, Display, "There were %d out of date files", OutOfDateFiles.Num());
}


/** Copies file. */
void FNetworkFileServerClientConnection::ProcessCopyFile( FArchive& In, FArchive& Out )
{
	FString To;
	FString From;
	In << To;	
	In << From;

	ConvertClientFilenameToServerFilename(To);
	ConvertClientFilenameToServerFilename(From);

	bool Success = Sandbox->CopyFile(*To, *From);
	Out << Success;
}


void FNetworkFileServerClientConnection::ProcessSetTimeStamp( FArchive& In, FArchive& Out )
{
	FString Filename;
	FDateTime Timestamp;
	In << Filename;	
	In << Timestamp;

	ConvertClientFilenameToServerFilename(Filename);

	Sandbox->SetTimeStamp(*Filename, Timestamp);

	// Need to sends something back otherwise the response won't get sent at all.
	bool Success = true;
	Out << Success;
}


void FNetworkFileServerClientConnection::ProcessSetReadOnly( FArchive& In, FArchive& Out )
{
	FString Filename;
	bool bReadOnly;
	In << Filename;	
	In << bReadOnly;

	ConvertClientFilenameToServerFilename(Filename);

	bool Success = Sandbox->SetReadOnly(*Filename, bReadOnly);
	Out << Success;
}


void FNetworkFileServerClientConnection::ProcessCreateDirectory( FArchive& In, FArchive& Out ) 
{
	FString Directory;
	In << Directory;

	ConvertClientFilenameToServerFilename(Directory);

	bool bSuccess = Sandbox->CreateDirectory(*Directory);
	Out << bSuccess;
}


void FNetworkFileServerClientConnection::ProcessDeleteDirectory( FArchive& In, FArchive& Out )
{
	FString Directory;
	In << Directory;

	ConvertClientFilenameToServerFilename(Directory);

	bool bSuccess = Sandbox->DeleteDirectory(*Directory);
	Out << bSuccess;
}


void FNetworkFileServerClientConnection::ProcessDeleteDirectoryRecursively( FArchive& In, FArchive& Out )
{
	FString Directory;
	In << Directory;

	ConvertClientFilenameToServerFilename(Directory);

	bool bSuccess = Sandbox->DeleteDirectoryRecursively(*Directory);
	Out << bSuccess;
}


void FNetworkFileServerClientConnection::ProcessToAbsolutePathForRead( FArchive& In, FArchive& Out )
{
	FString Filename;
	In << Filename;

	ConvertClientFilenameToServerFilename(Filename);

	Filename = Sandbox->ConvertToAbsolutePathForExternalAppForRead(*Filename);
	Out << Filename;
}


void FNetworkFileServerClientConnection::ProcessToAbsolutePathForWrite( FArchive& In, FArchive& Out )
{
	FString Filename;
	In << Filename;

	ConvertClientFilenameToServerFilename(Filename);

	Filename = Sandbox->ConvertToAbsolutePathForExternalAppForWrite(*Filename);
	Out << Filename;
}

static void AddDirectoriesToIgnore(const FString& RootDir, TArray<FString>& OutDirectoriesToSkip, TArray<FString>& OutDirectoriesToNotRecurse)
{
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Intermediate")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Documentation")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Extras")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Binaries")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Source")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Saved")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Plugins")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Programs")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Platforms")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Build")));
	OutDirectoriesToSkip.Add(FString(RootDir / TEXT("Restricted")));
	OutDirectoriesToNotRecurse.Add(FString(RootDir / TEXT("DerivedDataCache")));
}

static void ScanExtensionRootDirectory(FSandboxPlatformFile* Sandbox, const FString& RootDir, const TArray<FString>& RootDirectories, TMap<FString, FDateTime>& OutFileTimes)
{
	// Ensure that the path from the extension root to any containing root directory is in the FileTimes map
	for (int32 DirIndex = 0; DirIndex < RootDirectories.Num(); DirIndex++)
	{
		if (FPaths::IsUnderDirectory(RootDir, RootDirectories[DirIndex]))
		{
			FString PathUpwardSegment = RootDir;
			do 
			{
				OutFileTimes.Add(PathUpwardSegment, 0);
				PathUpwardSegment = FPaths::GetPath(PathUpwardSegment);
			} while (FPaths::IsUnderDirectory(PathUpwardSegment, RootDirectories[DirIndex]));
			break;
		}
	}

	TArray<FString> ExtensionDirectoriesToSkip;
	TArray<FString> ExtensionDirectoriesToNotRecurse;
	AddDirectoriesToIgnore(RootDir, ExtensionDirectoriesToSkip, ExtensionDirectoriesToNotRecurse);
	FLocalTimestampDirectoryVisitor ExtensionVisitor(*Sandbox, ExtensionDirectoriesToSkip, ExtensionDirectoriesToNotRecurse, true);
	Sandbox->IterateDirectory(*RootDir, ExtensionVisitor);
	OutFileTimes.Append(ExtensionVisitor.FileTimes);
}

bool FNetworkFileServerClientConnection::ProcessGetFileList( FArchive& In, FArchive& Out )
{
	// get the list of directories to process
 	TArray<FString> TargetPlatformNames;
	FString GameName;
	FString EngineRelativePath;
	FString GameRelativePath;
	FString EnginePlatformExtensionsRelativePath;
	FString ProjectPlatformExtensionsRelativePath;
	FString EnginePluginsRelativePath;
	FString ProjectPluginsRelativePath;
	TArray<FString> RootDirectories;
	TMap<FString,FString> CustomPlatformData;

	
	EConnectionFlags ConnectionFlags;

	FString ClientVersionInfo;
	FString TargetAddress;

	In << TargetPlatformNames;
	In << GameName;
	In << EngineRelativePath;
	In << GameRelativePath;
	In << EnginePlatformExtensionsRelativePath;
	In << ProjectPlatformExtensionsRelativePath;
	In << EnginePluginsRelativePath;
	In << ProjectPluginsRelativePath;
	In << RootDirectories;
	In << ConnectionFlags;
	In << ClientVersionInfo;
	In << TargetAddress;
	In << CustomPlatformData;

	if ( NetworkFileDelegates->NewConnectionDelegate.IsBound() )
	{
		bool bIsValidVersion = true;
		for ( const FString& TargetPlatform : TargetPlatformNames )
		{
			bIsValidVersion &= NetworkFileDelegates->NewConnectionDelegate.Execute(ClientVersionInfo, TargetPlatform );
		}
		if ( bIsValidVersion == false )
		{
			return false;
		}
	}

	const bool bIsStreamingRequest = (ConnectionFlags & EConnectionFlags::Streaming) == EConnectionFlags::Streaming;

	ConnectedPlatformName = TEXT("");
	ConnectedTargetPlatform = nullptr;
	ConnectedIPAddress = TEXT("");
	ConnectedTargetCustomData.Reset();


	// if we didn't find one (and this is a dumb server - no active platforms), then just use what was sent
	if (ActiveTargetPlatforms.Num() == 0)
	{
		ConnectedPlatformName = TargetPlatformNames[0];
	}
	// we only need to care about validating the connected platform if there are active targetplatforms
	else
	{
		// figure out the best matching target platform for the set of valid ones
		for (int32 TPIndex = 0; TPIndex < TargetPlatformNames.Num() && ConnectedPlatformName == TEXT(""); TPIndex++)
		{
			UE_LOGF(LogFileServer, Display, "    Possible Target Platform from client: %ls", *TargetPlatformNames[TPIndex]);

			// look for a matching target platform
			for (int32 ActiveTPIndex = 0; ActiveTPIndex < ActiveTargetPlatforms.Num(); ActiveTPIndex++)
			{
				UE_LOGF(LogFileServer, Display, "   Checking against: %ls", *ActiveTargetPlatforms[ActiveTPIndex]->PlatformName());
				if (ActiveTargetPlatforms[ActiveTPIndex]->PlatformName() == TargetPlatformNames[TPIndex])
				{
					bSendLowerCase = ActiveTargetPlatforms[ActiveTPIndex]->SendLowerCaseFilePaths();
					ConnectedPlatformName = ActiveTargetPlatforms[ActiveTPIndex]->PlatformName();
					ConnectedTargetPlatform = ActiveTargetPlatforms[ActiveTPIndex];
					ConnectedIPAddress = TargetAddress;
					ConnectedTargetCustomData = MoveTemp(CustomPlatformData);
					break;
				}
			}
		}

		// if we didn't find one, reject client and also print some warnings
		if (ConnectedPlatformName == TEXT(""))
		{
			// reject client we can't cook/compile shaders for you!
			UE_LOGF(LogFileServer, Warning, "Unable to find target platform for client, terminating client connection!");

			for (int32 TPIndex = 0; TPIndex < TargetPlatformNames.Num() && ConnectedPlatformName == TEXT(""); TPIndex++)
			{
				UE_LOGF(LogFileServer, Warning, "    Target platforms from client: %ls", *TargetPlatformNames[TPIndex]);
			}
			for (int32 ActiveTPIndex = 0; ActiveTPIndex < ActiveTargetPlatforms.Num(); ActiveTPIndex++)
			{
				UE_LOGF(LogFileServer, Warning, "    Active target platforms on server: %ls", *ActiveTargetPlatforms[ActiveTPIndex]->PlatformName());
			}
			return false;
		}
	}

	ConnectedEngineDir = EngineRelativePath;
	ConnectedProjectDir = GameRelativePath;
	ConnectedEnginePlatformExtensionsDir = EnginePlatformExtensionsRelativePath;
	ConnectedProjectPlatformExtensionsDir = ProjectPlatformExtensionsRelativePath;

	UE_LOGF(LogFileServer, Display, "    Connected EngineDir      = %ls", *ConnectedEngineDir);
	UE_LOGF(LogFileServer, Display, "        Local EngineDir      = %ls", *LocalEngineDir);
	UE_LOGF(LogFileServer, Display, "    Connected ProjectDir     = %ls", *ConnectedProjectDir);
	UE_LOGF(LogFileServer, Display, "        Local ProjectDir     = %ls", *LocalProjectDir);
	UE_LOGF(LogFileServer, Display, "    Connected EnginePlatformExtDir = %ls", *ConnectedEnginePlatformExtensionsDir);
	UE_LOGF(LogFileServer, Display, "        Local EnginePlatformExtDir = %ls", *LocalEnginePlatformExtensionsDir);
	UE_LOGF(LogFileServer, Display, "    Connected ProjectPlatformExtDir = %ls", *ConnectedProjectPlatformExtensionsDir);
	UE_LOGF(LogFileServer, Display, "        Local ProjectPlatformExtDir = %ls", *LocalProjectPlatformExtensionsDir);

	// Remap the root directories requested...
	for (int32 RootDirIdx = 0; RootDirIdx < RootDirectories.Num(); RootDirIdx++)
	{
		FString CheckRootDir = RootDirectories[RootDirIdx];
		ConvertClientFilenameToServerFilename(CheckRootDir);
		RootDirectories[RootDirIdx] = CheckRootDir;
	}

	// figure out the sandbox directory
	// @todo: This should use FPlatformMisc::SavedDirectory(GameName)
	FString SandboxDirectory;
	if (NetworkFileDelegates->SandboxPathOverrideDelegate.IsBound() )
	{
		SandboxDirectory = NetworkFileDelegates->SandboxPathOverrideDelegate.Execute();
		// if the sandbox directory delegate returns a path with the platform name in it then replace it :)
		SandboxDirectory.ReplaceInline(TEXT("[Platform]"), *ConnectedPlatformName);
	}
	else if ( FPaths::IsProjectFilePathSet() )
	{
		FString ProjectDir = FPaths::GetPath(FPaths::GetProjectFilePath());
		SandboxDirectory = FPaths::Combine(*ProjectDir, TEXT("Saved"), TEXT("Cooked"), *ConnectedPlatformName);

		// this is a workaround because the cooker and the networkfile server don't have access to eachother and therefore don't share the same Sandbox
		// the cooker in cook in editor saves to the EditorCooked directory
		if ( GIsEditor && !IsRunningCommandlet())
		{
			SandboxDirectory = FPaths::Combine(*ProjectDir, TEXT("Saved"), TEXT("EditorCooked"), *ConnectedPlatformName);
		}
		if( bIsStreamingRequest )
		{
			RootDirectories.Add(ProjectDir);
		}
	}
	else
	{
		if (FPaths::GetExtension(GameName) == FProjectDescriptor::GetExtension())
		{
			SandboxDirectory = FPaths::Combine(*FPaths::GetPath(GameName), TEXT("Saved"), TEXT("Cooked"), *ConnectedPlatformName);
		}
		else
		{
			//@todo: This assumes the game is located in the Unreal Root directory
			SandboxDirectory = FPaths::Combine(*FPaths::GetRelativePathToRoot(), *GameName, TEXT("Saved"), TEXT("Cooked"), *ConnectedPlatformName);
		}
	}
	// Convert to full path so that the sandbox wrapper doesn't re-base to Saved/Sandboxes
	SandboxDirectory = FPaths::ConvertRelativePathToFull(SandboxDirectory);

	// delete any existing one first, in case game name somehow changed and client is re-asking for files (highly unlikely)
	Sandbox.Reset();
	Sandbox = FSandboxPlatformFile::Create(false);
	Sandbox->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), *FString::Printf(TEXT("-sandbox=\"%s\""), *SandboxDirectory));

	GetSandboxRootDirectories(Sandbox.Get(), SandboxEngine, SandboxProject, SandboxEnginePlatformExtensions, SandboxProjectPlatformExtensions, LocalEngineDir, LocalProjectDir, LocalEnginePlatformExtensionsDir, LocalProjectPlatformExtensionsDir);

	UE_LOGF(LogFileServer, Display, "Getting files for %d directories, game = %ls, platform = %ls", RootDirectories.Num(), *GameName, *ConnectedPlatformName);
	UE_LOGF(LogFileServer, Display, "    Sandbox dir = %ls", *SandboxDirectory);

	for (int32 DumpIdx = 0; DumpIdx < RootDirectories.Num(); DumpIdx++)
	{
		UE_LOGF(LogFileServer, Display, "\t%ls", *(RootDirectories[DumpIdx]));
	}

	TArray<FString> DirectoriesToAlwaysStageAsUFS;
	if ( GConfig->GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("DirectoriesToAlwaysStageAsUFS"), DirectoriesToAlwaysStageAsUFS, GGameIni) )
	{
		for ( const auto& DirectoryToAlwaysStage : DirectoriesToAlwaysStageAsUFS )
		{
			RootDirectories.Add( DirectoryToAlwaysStage );
		}
	}

	// list of directories to skip
	TArray<FString> DirectoriesToSkip;
	TArray<FString> DirectoriesToNotRecurse;
	// @todo: This should really be FPlatformMisc::GetSavedDirForGame(ClientGameName), etc
	for (const FString& RootDir : RootDirectories)
	{
		AddDirectoriesToIgnore(RootDir, DirectoriesToSkip, DirectoriesToNotRecurse);
	}

	UE_LOGF(LogFileServer, Display, "Scanning server files for timestamps...");

	double FileScanStartTime = FPlatformTime::Seconds();

	// use the timestamp grabbing visitor (include directories)
	FLocalTimestampDirectoryVisitor Visitor(*Sandbox, DirectoriesToSkip, DirectoriesToNotRecurse, true);
	for (int32 DirIndex = 0; DirIndex < RootDirectories.Num(); DirIndex++)
	{
		bool bIsSubDirOfOtherRootDir = false;
		for (int32 OtherDirIndex = 0; OtherDirIndex < DirIndex; OtherDirIndex++)
		{
			if (OtherDirIndex == DirIndex)
				continue;

			if (FPaths::IsUnderDirectory(RootDirectories[DirIndex], RootDirectories[OtherDirIndex]))
			{
				bIsSubDirOfOtherRootDir = true;
				break;
			}
		}

		if (!bIsSubDirOfOtherRootDir)
		Sandbox->IterateDirectory(*RootDirectories[DirIndex], Visitor);
	}

	// Get PlatformDirectoryNames
	FString ServerEnginePlatformExtensionsRelativePath = EnginePlatformExtensionsRelativePath;
	ConvertClientFilenameToServerFilename(ServerEnginePlatformExtensionsRelativePath);
	FString ServerProjectPlatformExtensionsRelativePath = ProjectPlatformExtensionsRelativePath;
	ConvertClientFilenameToServerFilename(ServerProjectPlatformExtensionsRelativePath);

	TArray<FString> PlatformDirectoryNames;
	for (const FString& TargetPlatform : TargetPlatformNames)
	{
		FName IniPlatformName = PlatformInfo::FindPlatformInfo(*TargetPlatform)->IniPlatformName;
		const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName);
		PlatformDirectoryNames.Reserve(PlatformInfo.IniParentChain.Num() + PlatformInfo.AdditionalRestrictedFolders.Num() + 1);
		PlatformDirectoryNames.Add(IniPlatformName.ToString());
		for (const FString& PlatformName : PlatformInfo.AdditionalRestrictedFolders)
		{
			PlatformDirectoryNames.AddUnique(PlatformName);
		}
		for (const FString& PlatformName : PlatformInfo.IniParentChain)
		{
			PlatformDirectoryNames.AddUnique(PlatformName);
		}
	}

	// Traverse plugin directories
	TSet<FString> PlatformDirectoryNameSet;
	PlatformDirectoryNameSet.Append(PlatformDirectoryNames);
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	for (TSharedRef<IPlugin> Plugin : AllPlugins)
	{
		// First the base directory of the plugin.
		ScanExtensionRootDirectory(Sandbox.Get(), Plugin->GetBaseDir(), RootDirectories, Visitor.FileTimes);

		// Next the plugin extension directories of this plugin.
		TArray<FString> PluginExtensionDirs = Plugin->GetExtensionBaseDirs();
		for (const FString& ExtensionDir : PluginExtensionDirs)
		{
			// Scan for Platforms/X.  If X is not one of our platforms do not scan this extension directory.  
			// If X is one of our platforms or this extension is not Platforms restricted at all scan it.
			bool bFoundPlatforms = false;
			bool bDone = false;
			bool bWrongPlatform = false;
			FPathViews::IterateComponents(
				ExtensionDir,
				[&bFoundPlatforms, &bDone, &bWrongPlatform, &PlatformDirectoryNameSet](FStringView CurrentPathComponent)
				{
					if (!bFoundPlatforms)
					{
						if (CurrentPathComponent == FString("Platforms"))
						{
							bFoundPlatforms = true;
						}
					}
					else if (!bDone)
					{
						bWrongPlatform = !PlatformDirectoryNameSet.Contains(FString(CurrentPathComponent));
						bDone = true;
					}
					else
					{
						// Do nothing.
					}
				}
			);

			if (!bWrongPlatform)
			{
				ScanExtensionRootDirectory(Sandbox.Get(), ExtensionDir, RootDirectories, Visitor.FileTimes);
			}
		}
	}

	// Traverse platform extension directories
	for (const FString& PlatformDirectoryName : PlatformDirectoryNames)
	{
		ScanExtensionRootDirectory(Sandbox.Get(), ServerEnginePlatformExtensionsRelativePath / PlatformDirectoryName, RootDirectories, Visitor.FileTimes);
		ScanExtensionRootDirectory(Sandbox.Get(), ServerProjectPlatformExtensionsRelativePath / PlatformDirectoryName, RootDirectories, Visitor.FileTimes);
	}

	UE_LOGF(LogFileServer, Display, "Scanned server files, found %d files in %.2f seconds", Visitor.FileTimes.Num(), FPlatformTime::Seconds() - FileScanStartTime);

	// report the package version information
	// The downside of this is that ALL cooked data will get tossed on package version changes
	FPackageFileVersion PackageFileUnrealVersion = GPackageFileUEVersion;
	Out << PackageFileUnrealVersion;
	int32 PackageFileLicenseeUnrealVersion = GPackageFileLicenseeUEVersion;
	Out << PackageFileLicenseeUnrealVersion;

	// Send *our* engine and game dirs
	Out << LocalEngineDir;
	Out << LocalProjectDir;
	Out << LocalEnginePlatformExtensionsDir;
	Out << LocalProjectPlatformExtensionsDir;

	// return the files and their timestamps
	TMap<FString, FDateTime> FixedTimes = FixupSandboxPathsForClient(Visitor.FileTimes);
	Out << FixedTimes;

#if 0 // dump the list of files
	for ( const auto& FileTime : Visitor.FileTimes)
	{
		UE_LOGF(LogFileServer, Display, "Server list of files  %ls time %d", *FileTime.Key, *FileTime.Value.ToString() );
	}
#endif
	// Do it again, preventing access to non-cooked files
	if( bIsStreamingRequest == false )
	{
		TArray<FString> RootContentPaths;
		FPackageName::QueryRootContentPaths(RootContentPaths); 
		TArray<FString> ContentFolders;
		for (const auto& RootPath : RootContentPaths)
		{
			const FString& ContentFolder = FPackageName::LongPackageNameToFilename(RootPath);

			FString ConnectedContentFolder = ContentFolder;
			ConnectedContentFolder.ReplaceInline(*LocalEngineDir, *ConnectedEngineDir);
			ConnectedContentFolder.ReplaceInline(*LocalEnginePlatformExtensionsDir, *ConnectedEnginePlatformExtensionsDir);
			ConnectedContentFolder.ReplaceInline(*LocalProjectPlatformExtensionsDir, *ConnectedProjectPlatformExtensionsDir);

			int32 ReplaceCount = 0;

			// If one path is relative and the other isn't, convert both to absolute paths before trying to replace
			if (FPaths::IsRelative(LocalProjectDir) != FPaths::IsRelative(ConnectedContentFolder))
			{
				FString AbsoluteLocalGameDir = FPaths::ConvertRelativePathToFull(LocalProjectDir);
				FString AbsoluteConnectedContentFolder = FPaths::ConvertRelativePathToFull(ConnectedContentFolder);
				ReplaceCount = AbsoluteConnectedContentFolder.ReplaceInline(*AbsoluteLocalGameDir, *ConnectedProjectDir);
				if (ReplaceCount > 0)
				{
					ConnectedContentFolder = AbsoluteConnectedContentFolder;
				}
			}
			else
			{
				ReplaceCount = ConnectedContentFolder.ReplaceInline(*LocalProjectDir, *ConnectedProjectDir);
			}
			
			if (ReplaceCount == 0)
			{
				int32 GameDirOffset = ConnectedContentFolder.Find(ConnectedProjectDir, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (GameDirOffset != INDEX_NONE)
				{
					ConnectedContentFolder.RightChopInline(GameDirOffset, EAllowShrinking::No);
				}
			}

			ContentFolders.Add(ConnectedContentFolder);
		}
		Out << ContentFolders;

		// return the cached files and their timestamps
		// TODO: This second file list is now identical to the first.  This should be cleaned up in the future to not send two lists.
		Out << FixedTimes;
	}

	return true;
}

void FNetworkFileServerClientConnection::FileModifiedCallback( const FString& Filename)
{
	FScopeLock Lock(&ModifiedFilesSection);

	// do we care about this file???

	// translation here?
	ModifiedFiles.AddUnique(Filename);
}

void FNetworkFileServerClientConnection::ProcessHeartbeat( FArchive& In, FArchive& Out )
{
	TArray<FString> FixedupModifiedFiles;
	// Protect the array
	if (Sandbox)
	{
		FScopeLock Lock(&ModifiedFilesSection);
		
		for (const auto& ModifiedFile : ModifiedFiles)
		{
			FixedupModifiedFiles.Add(FixupSandboxPathForClient(ModifiedFile));
		}
		ModifiedFiles.Empty();
	}
	// return the list of modified files
	Out << FixedupModifiedFiles;

	

	// @todo: note the last received time, and toss clients that don't heartbeat enough!

	// @todo: Right now, there is no directory watcher adding to ModifiedFiles. It had to be pulled from this thread (well, the ModuleManager part)
	// We should have a single directory watcher that pushes the changes to all the connections - or possibly pass in a shared DirectoryWatcher
	// and have each connection set up a delegate (see p4 history for HandleDirectoryWatcherDirectoryChanged)
}


/* FStreamingNetworkFileServerConnection callbacks
 *****************************************************************************/

bool FNetworkFileServerClientConnection::PackageFile( FString& Filename, FString& TargetFilename, FArchive& Out )
{
	// get file timestamp and send it to client
	FDateTime ServerTimeStamp = Sandbox->GetTimeStamp(*Filename);

	// Disable access to outside the sandbox to prevent sending uncooked packages to the client
	const bool bSandboxOnly = bRestrictPackageAssetsToSandbox && FPackageName::IsPackageExtension(*FPaths::GetExtension(Filename, true));
	FSandboxOnlyScope _(*Sandbox, bSandboxOnly);

	FString AbsHostFile = Sandbox->ConvertToAbsolutePathForExternalAppForRead(*Filename);
	if (ConnectedTargetPlatform != nullptr && ConnectedTargetPlatform->CopyFileToTarget(ConnectedIPAddress, AbsHostFile, TargetFilename, ConnectedTargetCustomData))
	{
		Out << Filename;
		Out << ServerTimeStamp;
		// MAX_uint64 here indicates that it was copied already
		uint64 FileSize = MAX_uint64;
		Out << FileSize;

		return true;
	}

	TArray<uint8> Contents;
	// open file
	IFileHandle* File = Sandbox->OpenRead(*Filename);
	bool bRetVal = true;

	if (!File)
	{
		++PackageRequestsFailed;

		UE_LOGF(LogFileServer, Warning, "Opening file %ls failed", *Filename);
		ServerTimeStamp = FDateTime::MinValue(); // if this was a directory, this will make sure it is not confused with a zero byte file
		bRetVal = false;
	}
	else
	{
		++PackageRequestsSucceeded;

		if (!File->Size())
		{
			UE_LOGF(LogFileServer, Warning, "Sending empty file %ls....", *Filename);
		}
		else
		{
			if (IntFitsIn<int32, int64>(File->Size()))
			{
				int32 FileSize32 = static_cast<int32>(File->Size());

				FileBytesSent += FileSize32;
				// read it
				Contents.AddUninitialized(FileSize32);
				File->Read(Contents.GetData(), Contents.Num());
			}
			else
			{
				UE_LOGF(LogFileServer, Warning, "Unable to open %ls because it is too large", *Filename);
				bRetVal = false;
			}
		}

		// close it
		delete File;

		UE_LOGF(LogFileServer, Display, "Read %ls, %d bytes", *Filename, Contents.Num());
	}

	Out << Filename;
	Out << ServerTimeStamp;
	uint64 FileSize = Contents.Num();
	Out << FileSize;
	Out.Serialize(Contents.GetData(), FileSize);
	return bRetVal;
}

bool FNetworkFileServerClientConnection::ProcessSyncFile( FArchive& In, FArchive& Out )
{

	double StartTime;
	StartTime = FPlatformTime::Seconds();

	// get filename
	FString Filename;
	In << Filename;
	
	UE_LOGF(LogFileServer, Verbose, "Try sync file %ls", *Filename);

	FString ClientFilename = Filename;
	ConvertClientFilenameToServerFilename(Filename);
	
	//FString AbsFile(FString(*Sandbox->ConvertToAbsolutePathForExternalApp(*Filename)).MakeStandardFilename());
	// ^^ we probably in general want that filename, but for cook on the fly, we want the un-sandboxed name

	TArray<FString> NewUnsolictedFiles;

	NetworkFileDelegates->FileRequestDelegate.ExecuteIfBound(Filename, ConnectedPlatformName, NewUnsolictedFiles);

	FileRequestDelegateTime += 1000.0f * float(FPlatformTime::Seconds() - StartTime);
	StartTime = FPlatformTime::Seconds();
	

	for (int32 Index = 0; Index < NewUnsolictedFiles.Num(); Index++)
	{
		if (NewUnsolictedFiles[Index] != Filename)
		{
			UnsolictedFiles.AddUnique(NewUnsolictedFiles[Index]);
		}
	}

	bool bRetVal = PackageFile(Filename, ClientFilename, Out);

	PackageFileTime += 1000.0f * float(FPlatformTime::Seconds() - StartTime);

	return bRetVal;
}

FString FNetworkFileServerClientConnection::GetDescription() const 
{
	return FString("Client For " ) + ConnectedPlatformName;
}


bool FNetworkFileServerClientConnection::Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) 
{
	if (FParse::Command(&Cmd, TEXT("networkserverconnection")))
	{
		if (FParse::Command(&Cmd, TEXT("stats")))
		{

			Ar.Logf(TEXT("Network server connection %s stats\n"
				"FileRequestDelegateTime \t%fms \n"
				"PackageFileTime \t%fms \n"
				"UnsolicitedFilesTime \t%fms \n"
				"FileRequestCount \t%d \n"
				"UnsolicitedFilesCount \t%d \n"
				"PackageRequestsSucceeded \t%d \n"
				"PackageRequestsFailed \t%d \n"
				"FileBytesSent \t%d \n"),
				*GetDescription(),
				FileRequestDelegateTime,
				PackageFileTime,
				UnsolicitedFilesTime,
				FileRequestCount,
				UnsolicitedFilesCount,
				PackageRequestsSucceeded,
				PackageRequestsFailed,
				FileBytesSent);

			// there could be multiple network platform files so let them all report their stats
			return false;
		}
	}
	return false;
}

