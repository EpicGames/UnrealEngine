// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "INetworkFileSystemModule.h"

class FSandboxPlatformFile;
class ITargetPlatform;


/**
 * This class processes all incoming messages from the client.
 */
class FNetworkFileServerClientConnection : public FSelfRegisteringExec
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket - The client socket to use.
	 * @param NetworkFileDelegates- delegates the client calls when events from the client happen
	 */
	FNetworkFileServerClientConnection(const FNetworkFileServerOptions& InFileServerOptions);

	/**
	 * Destructor.
	 */
	virtual ~FNetworkFileServerClientConnection( );

public:

	/**
	 * Processes the given payload.
	 *
	 * @param Ar An archive containing the payload data.
	 * @param Out An archive that will contain the processed data.
	 */
	bool ProcessPayload(FArchive& Ar);

	/**
	 * Gets the client connection's description.
	 *
	 * @return Description string.
	 */
	FString GetDescription() const;

protected:

	/**
	 *	Convert the given filename from the client to the server version of it
	 *	NOTE: Potentially modifies the input FString!!!!
	 *
	 *	@param	FilenameToConvert		Upon input, the client version of the filename. After the call, the server version
	 */
	void ConvertClientFilenameToServerFilename(FString& FilenameToConvert);

	/**
	 *	Convert the given filename from the an arbitrary local representation to the normalized server version of it
	 *	NOTE: Potentially modifies the input FString!!!!
	 *
	 *	@param	FilenameToConvert		Upon input, the filename. After the call, the server version
	 */
	void ConvertLocalFilenameToServerFilename(FString& FilenameToConvert);

	/**
	 *	Convert the given filename from the server to the client version of it
	 *	NOTE: Potentially modifies the input FString!!!!
	 *
	 *	@param	FilenameToConvert		Upon input, the server version of the filename. After the call, the client version
	 */
	// void ConvertServerFilenameToClientFilename(FString& FilenameToConvert);
	
	/** Opens a file for reading or writing. */
	void ProcessOpenFile(FArchive& In, FArchive& Out, bool bIsWriting);

	/** Reads from file. */
	void ProcessReadFile(FArchive& In, FArchive& Out);

	/** Writes to file. */
	void ProcessWriteFile(FArchive& In, FArchive& Out);

	/** Seeks in file. */
	void ProcessSeekFile(FArchive& In, FArchive& Out);

	/** Closes file handle and removes it from the open handles list. */
	void ProcessCloseFile(FArchive& In, FArchive& Out);

	/** Gets info on the specified file. */
	void ProcessGetFileInfo(FArchive& In, FArchive& Out);

	/** Moves file. */
	void ProcessMoveFile(FArchive& In, FArchive& Out);

	/** Deletes file. */
	void ProcessDeleteFile(FArchive& In, FArchive& Out);

	/** Copies file. */
	void ProcessCopyFile(FArchive& In, FArchive& Out);

	/** Sets file timestamp. */
	void ProcessSetTimeStamp(FArchive& In, FArchive& Out);

	/** Sets read only flag. */
	void ProcessSetReadOnly(FArchive& In, FArchive& Out);

	/** Creates directory. */
	void ProcessCreateDirectory(FArchive& In, FArchive& Out);

	/** Deletes directory. */
	void ProcessDeleteDirectory(FArchive& In, FArchive& Out);

	/** Deletes directory recursively. */
	void ProcessDeleteDirectoryRecursively(FArchive& In, FArchive& Out);

	/** ConvertToAbsolutePathForExternalAppForRead. */
	void ProcessToAbsolutePathForRead(FArchive& In, FArchive& Out);

	/** ConvertToAbsolutePathForExternalAppForWrite. */
	void ProcessToAbsolutePathForWrite(FArchive& In, FArchive& Out);

	/** Reposts local files. */
	void ProcessReportLocalFiles(FArchive& In, FArchive& Out);

	/** Walk over a set of directories, and get all files (recursively) in them, along with their timestamps. */
	bool ProcessGetFileList(FArchive& In, FArchive& Out);

	/** Heartbeat. */
	void ProcessHeartbeat(FArchive& In, FArchive& Out);

	/** 
	 * Finds open file handle by its ID.
	 *
	 * @param HandleId
	 * @return Pointer to the file handle, NULL if the specified handle id doesn't exist.
	 */
	FORCEINLINE IFileHandle* FindOpenFile( uint64 HandleId )
	{
		IFileHandle** OpenFile = OpenFiles.Find(HandleId);

		return OpenFile ? *OpenFile : NULL;
	}

	bool PackageFile( FString& Filename, FString& TargetFilename, FArchive& Out);

	/**
	 * Processes a heartbeat message.
	 *
	 * @param In -
	 * @param Out -
	 */
	bool ProcessSyncFile( FArchive& In, FArchive& Out );


	virtual bool SendPayload( TArray<uint8> &Out ) = 0; 

	/**
	 * When a file is modified this callback is triggered
	 *  cleans up any cached information about the file and notifies client
	 * 
	 * @param Filename of the file which has been modified
	 */
	void FileModifiedCallback( const FString& Filename );
	
	
	virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;


	/**
	 * Convert a path to a sandbox path and translate so the client can understand it 
	 *
	 * @param Filename to convert
	 * @param bLowerCaseFiles conver the file name to all lower case
	 * @return Resulting fixed up path
	 */
	FString FixupSandboxPathForClient(const FString& Filename);

	/**
	* Convert a path to a sandbox path and translate so the client can understand it
	*
	* @param Filename to convert
	* @param bLowerCaseFiles conver the file name to all lower case
	* @return Resulting fixed up path
	*/
	TMap<FString,FDateTime> FixupSandboxPathsForClient(const TMap<FString, FDateTime>& SandboxPaths);
	
private:

	// Hold the name of the currently connected platform.
	FString ConnectedPlatformName;

	// Hold the ip address of the currently connected platform.
	FString ConnectedIPAddress;

	// Hold the engine directory from the connected platform.
	FString ConnectedEngineDir;

	// Hold the game directory from the connected platform.
	FString ConnectedProjectDir;

	// Hold the engine platform extensions directory from the connected platform.
	FString ConnectedEnginePlatformExtensionsDir;

	// Hold the project platform extensions directory from the connected platform.
	FString ConnectedProjectPlatformExtensionsDir;

	// Hold the sandbox engine directory for the connected platform
	FString SandboxEngine;

	// hold the sandbox game directory for the connected platform
	FString SandboxProject;

	// hold the engine sandbox platform extensions directory for the connected platform
	FString SandboxEnginePlatformExtensions;

	// hold the project sandbox platform extensions directory for the connected platform
	FString SandboxProjectPlatformExtensions;

	// Should we send the filenames in lowercase
	bool bSendLowerCase;

	// Holds the last assigned handle id (0 = invalid).
	uint64 LastHandleId;

	// Holds the list of files found by the directory watcher.
	TArray<FString> ModifiedFiles;

	// Holds a critical section to protect the ModifiedFiles array.
	FCriticalSection ModifiedFilesSection;

	// Holds all currently open file handles.
	TMap<uint64, IFileHandle*> OpenFiles;

	// Holds the file interface for local (to the server) files - all file ops MUST go through here, NOT IFileManager.
	TUniquePtr<FSandboxPlatformFile> Sandbox;

	// Holds the list of unsolicited files to send in separate packets.
	TArray<FString> UnsolictedFiles;

	// Holds the list of directories being watched.
	TArray<FString> WatchedDirectories;

	// Local path to the engine directory
	FString LocalEngineDir;

	// Absolute local path to the engine directory
	FString LocalEngineDirAbs;

	// Local path to the project directory
	FString LocalProjectDir;

	// Absolute local path to the project directory
	FString LocalProjectDirAbs;

	// Local path to the engine platform extensions directory
	FString LocalEnginePlatformExtensionsDir;

	// Absolute local path to the engine platform extensions directory
	FString LocalEnginePlatformExtensionsDirAbs;

	// Local path to the project platform extensions directory
	FString LocalProjectPlatformExtensionsDir;

	// Absolute local path to the project platform extensions directory
	FString LocalProjectPlatformExtensionsDirAbs;

	const FNetworkFileDelegateContainer* NetworkFileDelegates;

	// cached copy of the active target platforms (if any)
	const TArray<ITargetPlatform*>& ActiveTargetPlatforms;

	// connected TargetPlatform object, if ActiveTargetPlatforms existed
	ITargetPlatform* ConnectedTargetPlatform;

	// custom key-value pair data for the curently connected target platform
	TMap<FString,FString> ConnectedTargetCustomData;

	// whether to restrict sending package assets from outside the sandbox
	bool bRestrictPackageAssetsToSandbox;

	//////////////////////////////////////////////////////////////////////////
	//stats
	double FileRequestDelegateTime;
	double PackageFileTime;
	double UnsolicitedFilesTime;

	int32 FileRequestCount;
	int32 UnsolicitedFilesCount;
	int32 PackageRequestsSucceeded;
	int32 PackageRequestsFailed;
	int32 FileBytesSent;
	
};
