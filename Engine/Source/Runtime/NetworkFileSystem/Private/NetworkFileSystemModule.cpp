// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "INetworkFileSystemModule.h"
#include "NetworkFileSystemLog.h"
#include "NetworkFileServer.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "CookOnTheFlyNetServer.h"
#include "IPAddress.h"

DEFINE_LOG_CATEGORY(LogFileServer);


/**
 * Implements the NetworkFileSystem module.
 */
class FNetworkFileSystemModule
	: public INetworkFileSystemModule
{
public:

	// INetworkFileSystemModule interface

	virtual INetworkFileServer* CreateNetworkFileServer( bool bLoadTargetPlatforms, int32 Port, FNetworkFileDelegateContainer NetworkFileDelegateContainer, const ENetworkFileServerProtocol Protocol ) const
	{
		FNetworkFileServerOptions FileServerOptions;
		FileServerOptions.Protocol = Protocol;
		FileServerOptions.Port = Port;
		FileServerOptions.Delegates = MoveTemp(NetworkFileDelegateContainer);
		FileServerOptions.bRestrictPackageAssetsToSandbox = false; 
		
		return CreateNetworkFileServer(MoveTemp(FileServerOptions), bLoadTargetPlatforms);
	}
	
	virtual INetworkFileServer* CreateNetworkFileServer(FNetworkFileServerOptions FileServerOptions, bool bLoadTargetPlatforms) const override
	{
		if (bLoadTargetPlatforms)
		{
			ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();

			// if we didn't specify a target platform then use the entire target platform list (they could all be possible!)
			FString Platforms;
			if (FParse::Value(FCommandLine::Get(), TEXT("TARGETPLATFORM="), Platforms))
			{
				FileServerOptions.TargetPlatforms =  TPM.GetActiveTargetPlatforms();
			}
			else
			{
				FileServerOptions.TargetPlatforms = TPM.GetTargetPlatforms();
			}
		}

		switch (FileServerOptions.Protocol)
		{
		case NFSP_Tcp:
		case NFSP_Platform:
			UE::Cook::FCookOnTheFlyNetworkServerOptions CookOnTheFlyServerOptions;
			if (FileServerOptions.Protocol == NFSP_Tcp)
			{
				CookOnTheFlyServerOptions.Protocol = UE::Cook::ECookOnTheFlyNetworkServerProtocol::Tcp;
			}
			else
			{
				CookOnTheFlyServerOptions.Protocol = UE::Cook::ECookOnTheFlyNetworkServerProtocol::Platform;
			}
			CookOnTheFlyServerOptions.TargetPlatforms = FileServerOptions.TargetPlatforms;
			CookOnTheFlyServerOptions.Port = FileServerOptions.Port;
			UE::Cook::ICookOnTheFlyNetworkServerModule& CookOnTheFlyNetworkServerModule = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyNetworkServerModule>(TEXT("CookOnTheFlyNetServer"));
			TSharedPtr<UE::Cook::ICookOnTheFlyNetworkServer> CookOnTheFlyNetworkServer = CookOnTheFlyNetworkServerModule.CreateServer(CookOnTheFlyServerOptions);
			if (!CookOnTheFlyNetworkServer)
			{
				return nullptr;
			}
			UE_LOGF(LogFileServer, Display, "Unreal Network File Server starting up...");
			if (CookOnTheFlyNetworkServer->Start())
			{
				TArray<TSharedPtr<FInternetAddr>> ListenAddresses;
				if (CookOnTheFlyNetworkServer->GetAddressList(ListenAddresses))
				{
					UE_LOGF(LogFileServer, Display, "Unreal Network File Server is ready for client connections on %ls!", *ListenAddresses[0]->ToString(true));
				}
				return new FNetworkFileServer(MoveTemp(FileServerOptions), CookOnTheFlyNetworkServer.ToSharedRef());
			}
			break;
		}

		return nullptr;
	}

	virtual INetworkFileServer* CreateNetworkFileServer(TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> CookOnTheFlyNetworkServer, FNetworkFileDelegateContainer Delegates) const override
	{
		FNetworkFileServerOptions FileServerOptions;
		FileServerOptions.Delegates = MoveTemp(Delegates);
		FileServerOptions.bRestrictPackageAssetsToSandbox = true;
		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
		FString Platforms;
		if (FParse::Value(FCommandLine::Get(), TEXT("TARGETPLATFORM="), Platforms))
		{
			FileServerOptions.TargetPlatforms = TPM.GetActiveTargetPlatforms();
		}
		else
		{
			FileServerOptions.TargetPlatforms = TPM.GetTargetPlatforms();
		}
		return new FNetworkFileServer(MoveTemp(FileServerOptions), CookOnTheFlyNetworkServer);
	}
};


IMPLEMENT_MODULE(FNetworkFileSystemModule, NetworkFileSystem);
