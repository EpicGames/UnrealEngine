// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FInternetAddr;

/**
 * Interface for network file servers.
 */
class INetworkFileServer
{
public:
	/**
	 * 
	 * Returns Whether the network server was able to successfully start or not. 
	 *
	 * @return true on success, false otherwise.
	 */
	 virtual bool IsItReadyToAcceptConnections(void) const = 0;

	 /**
	  * Gets the list of local network addresses that the file server listens on.
	  *
	  * @param OutAddresses - Will hold the address list.
	  *
	  * @return true on success, false otherwise.
	  */
	virtual bool GetAddressList( TArray<TSharedPtr<FInternetAddr> >& OutAddresses ) const = 0;


	/**
	  * Gets the list of local network addresses that the file server listens on.
	  *
	  * @param OutAddresses - Will hold the address list.
	  *
	  * @return true on success, false otherwise.
	  */
	virtual FString GetSupportedProtocol() const = 0;

	/**
	 * Gets the number of active connections.
	 *
	 * @return The number of connections.
	 */
	virtual int32 NumConnections(void) const = 0;

	/**
	 * Shuts down the file server.
	 */
	virtual void Shutdown(void) = 0;

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~INetworkFileServer(void) = default;
};
