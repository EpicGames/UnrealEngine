// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ShaderCompiler.h"

class INetworkFileServer;
namespace UE::Cook
{
	class ICookOnTheFlyNetworkServer;
}

/**
 * Delegate type for handling file requests from a network client.
 *
 * The first parameter is the name of the requested file.
 * The second parameter will hold the list of unsolicited files to send back.
 */
DECLARE_DELEGATE_ThreeParams(FFileRequestDelegate, FString&, const FString&, TArray<FString>&);

/**
 * Delegate type for handling shader recompilation requests from a network client.
 */
DECLARE_DELEGATE_OneParam(FRecompileShadersDelegate, const FShaderRecompileData&);

/**
 * Delegate which returns an override for the sandbox path
 */
DECLARE_DELEGATE_RetVal( FString, FSandboxPathDelegate);

/**
 * Delegate which is called when an outside system modifies a file
 */
DECLARE_MULTICAST_DELEGATE_OneParam( FOnFileModifiedDelegate, const FString& );


/**
 * Delegate which is called when a new connection is made to a file server client
 * 
 * @param 1 Version string
 * @param 2 Platform name
 * @return return false if the connection should be destroyed
 */
DECLARE_DELEGATE_RetVal_TwoParams( bool, FNewConnectionDelegate, const FString&, const FString& );


// container struct for delegates which the network file system uses
struct FNetworkFileDelegateContainer
{
public:
	FNetworkFileDelegateContainer() : 
		NewConnectionDelegate(nullptr), 
		SandboxPathOverrideDelegate(nullptr),
		FileRequestDelegate(nullptr),
		RecompileShadersDelegate(nullptr),
		OnFileModifiedCallback(nullptr)
	{}
	FNewConnectionDelegate NewConnectionDelegate; 
	FSandboxPathDelegate SandboxPathOverrideDelegate;
	FFileRequestDelegate FileRequestDelegate;
	FRecompileShadersDelegate RecompileShadersDelegate;

	FOnFileModifiedDelegate* OnFileModifiedCallback; // this is called from other systems to notify the network file system that a file has been modified hence the terminology callback
};

enum ENetworkFileServerProtocol
{
	NFSP_Tcp,
	NFSP_Http,

	/**
	 * Platform-specific type of connection between a target device and a host pc.
	 * 
	 * It potentially offers performance benefits compared to generic networking protocols but is
	 * supported by only some of the platforms.
	 */
	NFSP_Platform,
};

// Network file server options
struct FNetworkFileServerOptions
{
	/* File server protocol*/
	ENetworkFileServerProtocol Protocol = NFSP_Tcp;

	/* The port number to bind to (-1 = default port, 0 = any available port) */
	int32 Port = INDEX_NONE;

	/* Optional delegates to be invoked when a file is requested by a client */
	FNetworkFileDelegateContainer Delegates;

	/* Active target platform(s) */
	TArray<ITargetPlatform*> TargetPlatforms;

	/* When running cook on the fly this options restricts package assets being sent from the project content folder */
	bool bRestrictPackageAssetsToSandbox = false;
};

/**
 * Interface for network file system modules.
 */
class INetworkFileSystemModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a new network file server.
	 *
	 * @param InPort The port number to bind to (-1 = default port, 0 = any available port).
	 * @param Streaming Whether it should be a streaming server.
	 * @param InFileRequestDelegate An optional delegate to be invoked when a file is requested by a client.
	 * @param InRecompileShadersDelegate An optional delegate to be invoked when shaders need to be recompiled.
	 *
	 * @return The new file server, or nullptr if creation failed.
	 */
	virtual INetworkFileServer* CreateNetworkFileServer( bool bLoadTargetPlatforms, int32 Port = -1, FNetworkFileDelegateContainer InNetworkFileDelegateContainer = FNetworkFileDelegateContainer(), const ENetworkFileServerProtocol Protocol = NFSP_Tcp ) const = 0;

    /**
	 * Creates a new network file server.
	 *
	 * @param FileServerOptions File server options
	 * @param bLoadTargetPlatforms If true, gets the target platform from the command line or all available target platforms
	 *
	 * @return The new file server, or nullptr if creation failed.
	 */
	virtual INetworkFileServer* CreateNetworkFileServer(FNetworkFileServerOptions FileServerOptions, bool bLoadTargetPlatforms) const = 0;

	/**
	 * Creates a new network file server for COTF.
	 *
	 * @param CookOnTheFlyNetworkServer CookOnTheFly network server
	 * @param Delegates Delegates to be invoked when a file is requested by a client.
	 *
	 * @return The new file server, or nullptr if creation failed.
	*/
	virtual INetworkFileServer* CreateNetworkFileServer(TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> CookOnTheFlyNetworkServer, FNetworkFileDelegateContainer Delegates) const
	{
		return nullptr;
	}

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~INetworkFileSystemModule( ) = default;
};
